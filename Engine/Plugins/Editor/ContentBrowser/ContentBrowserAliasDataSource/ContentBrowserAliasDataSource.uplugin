{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Content Browser - Alias Data Source",
	"Description": "Data Source plugin providing allowing Content Browser items to appear in other directories other than their original location",
	"Category": "Content Browser",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ContentBrowserAliasDataSource",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ContentBrowserAssetDataSource",
			"Enabled": true
		}
	]
}