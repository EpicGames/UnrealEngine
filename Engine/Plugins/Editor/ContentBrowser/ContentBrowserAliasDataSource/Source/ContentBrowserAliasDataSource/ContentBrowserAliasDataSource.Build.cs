// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ContentBrowserAliasDataSource : ModuleRules
	{
		public ContentBrowserAliasDataSource(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "CBADS";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AssetRegistry",
					"Core",
					"CoreUObject",
					"ContentBrowserAssetDataSource",
					"ContentBrowserData"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"Engine"
				}
			);

			// This module needs AutoRTFM so disable the auto disable since its in a plugin.
			// The call to the delegate `OnObjectPropertyChanged` calls into here.
			bDisableAutoRTFMInstrumentation = false;
		}
	}
}
