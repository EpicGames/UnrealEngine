// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAliasDataSource.h"
#include "ContentBrowserAliasDataSourceModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserAssetDataCore.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserModule.h"

#include "AssetToolsModule.h"
#include "CollectionManagerModule.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "IContentBrowserDataModule.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserAliasDataSource)

DEFINE_LOG_CATEGORY(LogContentBrowserAliasDataSource);

namespace ContentBrowserAliasDataSource
{
	FAutoConsoleCommand LogAliasesCommand = FAutoConsoleCommand(
		TEXT("ContentBrowser.LogAliases"),
		TEXT("Logs all content browser aliases"),
		FConsoleCommandWithArgsDelegate::CreateLambda(
			[](const TArray<FString>& /*Args*/)
			{
				if (UContentBrowserAliasDataSource* AliasDataSource = FModuleManager::Get().LoadModuleChecked<FContentBrowserAliasDataSourceModule>("ContentBrowserAliasDataSource").TryGetAliasDataSource())
				{
					AliasDataSource->LogAliases();
				}
			}));

	template <typename AliasType>
	FName GetAliasName(const AliasType&);

	template <>
	FName GetAliasName<FName>(const FName& Alias)
	{
		return Alias;
	}

	template <>
	FName GetAliasName<FContentBrowserLocalizedAlias>(const FContentBrowserLocalizedAlias& Alias)
	{
		return Alias.Alias;
	}

	template <typename AliasType>
	FText GetAliasDisplayNameOverride(const AliasType&);

	template <>
	FText GetAliasDisplayNameOverride<FName>(const FName& Alias)
	{
		return FText::GetEmpty();
	}

	template <>
	FText GetAliasDisplayNameOverride<FContentBrowserLocalizedAlias>(const FContentBrowserLocalizedAlias& Alias)
	{
		return Alias.DisplayName;
	}
}

FName UContentBrowserAliasDataSource::AliasTagName = "ContentBrowserAliases";

void UContentBrowserAliasDataSource::Initialize(const bool InAutoRegister)
{
	Super::Initialize(InAutoRegister);
	if (GIsEditor && !IsRunningCommandlet())
	{
		AssetRegistry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		AssetTools = &FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		ContentBrowserModule = &FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		AssetRegistry->OnAssetAdded().AddUObject(this, &UContentBrowserAliasDataSource::OnAssetAdded);
		AssetRegistry->OnAssetRemoved().AddUObject(this, &UContentBrowserAliasDataSource::OnAssetRemoved);
		AssetRegistry->OnAssetUpdated().AddUObject(this, &UContentBrowserAliasDataSource::OnAssetUpdated);
		FCoreUObjectDelegates::OnAssetLoaded.AddUObject(this, &UContentBrowserAliasDataSource::OnAssetLoaded);
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UContentBrowserAliasDataSource::OnObjectPropertyChanged);
	}
}

void UContentBrowserAliasDataSource::Shutdown()
{
	if (FModuleManager::Get().IsModuleLoaded(AssetRegistryConstants::ModuleName) && AssetRegistry)
	{
		AssetRegistry->OnAssetAdded().RemoveAll(this);
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
		AssetRegistry->OnAssetUpdated().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	Super::Shutdown();
}

void UContentBrowserAliasDataSource::BuildRootPathVirtualTree()
{
	Super::BuildRootPathVirtualTree();

	PathTree.EnumerateSubPaths("/", [this](FName Path)
	{
		RootPathAdded(FNameBuilder(Path));
		return true;
	}, false);
}

void UContentBrowserAliasDataSource::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserAliasDataSource::CompileFilter);

	UContentBrowserAssetDataSource::FAssetFilterInputParams Params;
	if (UContentBrowserAssetDataSource::PopulateAssetFilterInputParams(Params, this, AssetRegistry, InFilter, OutCompiledFilter, &FCollectionManagerModule::GetModule().Get(), &FilterCache))
	{
		// Use the DataSource's custom PathTree instead of the AssetRegistry
		const bool bCreatedPathFilter = UContentBrowserAssetDataSource::CreatePathFilter(Params, InPath, InFilter, OutCompiledFilter, [this](FName Path, TFunctionRef<bool(FName)> Callback, bool bRecursive)
		{
			PathTree.EnumerateSubPaths(Path, Callback, bRecursive);
		});

		if (bCreatedPathFilter)
		{
			auto CustomSubPathEnumeration = [this, &Params](FName Path, TFunctionRef<bool(FName)> Callback, bool bRecursive)
				{
					// Same as CreatePathFilter - CompileFilter calls EnumerateSubPaths internally so this needs to intercept
					// the filter compilation and use its own PathTree to generate the sub paths.
					AssetRegistry->EnumerateSubPaths(Path, Callback,bRecursive);
					PathTree.EnumerateSubPaths(Path, Callback, bRecursive);
				};

			auto CustomCollectionEnumeration = [this](const FCollectionRef& Collection, ECollectionRecursionFlags::Flags RecursionMode, TFunctionRef<void(const FSoftObjectPath&)> Callback)
				{
					if (!bFilterShouldMatchCollectionContent)
					{
						return;
					}

					TArray<FSoftObjectPath> ObjectPaths;
					Collection.Container->GetObjectsInCollection(Collection.Name, Collection.Type, ObjectPaths, RecursionMode);

					for (const FSoftObjectPath& ObjectPath : ObjectPaths)
					{
						if (const TArray<FName>* FoundAliases = AliasesForObjectPath.Find(ObjectPath))
						{
							for (const FName Alias : *FoundAliases)
							{
								if (const FAliasData* AliasData = AllAliases.Find(FContentBrowserUniqueAlias(ObjectPath, Alias)))
								{
									Callback(FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(AliasData->PackageName, AliasData->AssetData.AssetName)));
								}
							}
						}
					}
				};

			UContentBrowserAssetDataSource::FSubPathEnumerationFunc CustomSubPathEnumerationRef = CustomSubPathEnumeration;
			UContentBrowserAssetDataSource::FCollectionEnumerationFunc CustomCollectionEnumerationRef = CustomCollectionEnumeration;
			UContentBrowserAssetDataSource::CreateAssetFilter(Params, InPath, InFilter, OutCompiledFilter, &CustomSubPathEnumerationRef, &CustomCollectionEnumerationRef);
		}
	}
}

void UContentBrowserAliasDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return;
	}

	const FContentBrowserCompiledAssetDataFilter* AssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledAssetDataFilter>();
	if (!AssetDataFilter)
	{
		return;
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		// Use the DataSource's custom PathTree instead of the AssetRegistry
		auto EnumerateSubPaths = [this](FName Path, TFunctionRef<bool(FName)> Callback, bool bRecursive)
		{
			PathTree.EnumerateSubPaths(Path, Callback, bRecursive);
		};
		auto CreateFolderItem = [this](FName Path) -> FContentBrowserItemData
		{
			return CreateAssetFolderItem(Path);
		};
		UContentBrowserAssetDataSource::EnumerateFoldersMatchingFilter(this, AssetDataFilter, TGetOrEnumerateSink(InCallback), EnumerateSubPaths, CreateFolderItem);
	}

	if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !AssetDataFilter->bFilterExcludesAllAssets && !AssetDataFilter->InclusiveFilter.IsEmpty())
	{
		if (AssetDataFilter->InclusiveFilter.PackagePaths.Num() > 0)
		{
			// Find all aliases for each requested package path and check if it passes both
			// the inclusive and exclusive filters.
			for (const FName PackagePath : AssetDataFilter->InclusiveFilter.PackagePaths)
			{
				const TArray<FContentBrowserUniqueAlias>* Aliases = AliasesInPackagePath.Find(PackagePath);
				if (Aliases)
				{
					for (const FContentBrowserUniqueAlias& Alias : *Aliases)
					{
						const FAliasData& AliasData = AllAliases[Alias];
						if (DoesAliasPassFilter(AliasData, *AssetDataFilter))
						{
							if (!InCallback(CreateAssetFileItem(Alias)))
							{
								return;
							}
						}
					}
				}
			}
		}
		else
		{
			// If no package paths are requested, do the same as above for all aliases
			for (const TPair<FContentBrowserUniqueAlias, FAliasData>& Pair : AllAliases)
			{
				if (DoesAliasPassFilter(Pair.Value, *AssetDataFilter))
				{
					if (!InCallback(CreateAssetFileItem(Pair.Key)))
					{
						return;
					}
				}
			}
		}
	}
}

void UContentBrowserAliasDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return;
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
	{
		if (PathTree.PathExists(InternalPath))
		{
			InCallback(CreateAssetFolderItem(InternalPath));
		}
	}

	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
	{
		if (const FContentBrowserUniqueAlias* Alias = AliasForInternalPath.Find(InternalPath))
		{
			FContentBrowserItemData Item = CreateAssetFileItem(*Alias);
			if (ensureMsgf(Item.IsValid(), TEXT("Could not find alias data for: %s"), *InternalPath.ToString()))
			{
				InCallback(MoveTemp(Item));
			}
		}
	}
}

void UContentBrowserAliasDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !AliasesForObjectPath.IsEmpty())
	{
		auto EnumerateItemsAtObjectPath = [this, &InCallback](const FSoftObjectPath& ObjectPath)
		{
			if (ObjectPath.IsValid())
			{
				if (const TArray<FName>* FoundAliases = AliasesForObjectPath.Find(ObjectPath))
				{
					for (const FName Alias : *FoundAliases)
					{
						FContentBrowserItemData Item = CreateAssetFileItem(FContentBrowserUniqueAlias(ObjectPath, Alias));
						if (ensureMsgf(Item.IsValid(), TEXT("Could not find alias data for: %s"), *Alias.ToString()))
						{
							InCallback(MoveTemp(Item));
						}
					}
					return true;
				}
			}
			return false;
		};

		// Could be an object path.
		{
			FSoftObjectPath ObjectPath(FPackageName::ExportTextPathToObjectPath(InPath));

			if (EnumerateItemsAtObjectPath(ObjectPath))
			{
				return;
			}
		}

		// Could be a package name or file name.
		if (ensure(AssetRegistry))
		{
			FName PackageName;
			if (FPackageName::IsValidLongPackageName(InPath))
			{
				PackageName = FName(InPath, FNAME_Find);
			}
			else
			{
				FNameBuilder PackageNameBuilder;
				if (FPackageName::TryConvertFilenameToLongPackageName(InPath, PackageNameBuilder))
				{
					PackageName = FName(PackageNameBuilder, FNAME_Find);
				}
			}

			if (!PackageName.IsNone())
			{
				TArray<FAssetData> Assets;
				if (AssetRegistry->GetAssetsByPackageName(PackageName, Assets))
				{
					for (const FAssetData& AssetData : Assets)
					{
						EnumerateItemsAtObjectPath(AssetData.GetSoftObjectPath());
					}
				}
			}
		}
	}
}

bool UContentBrowserAliasDataSource::IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags, const FContentBrowserFolderContentsFilter& ContentsFilter)
{
	FName InternalPath;
	return TryConvertVirtualPathToInternal(Path, InternalPath) && PathTree.PathExists(InternalPath);
}

bool UContentBrowserAliasDataSource::DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return false;
	}

	const FContentBrowserCompiledAssetDataFilter* AssetDataFilter = FilterList->FindFilter<FContentBrowserCompiledAssetDataFilter>();
	if (!AssetDataFilter)
	{
		return false;
	}

	switch (InItem.GetItemType())
	{
	case EContentBrowserItemFlags::Type_Folder:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
		{
			return UContentBrowserAssetDataSource::DoesItemPassFolderFilter(this, InItem, *AssetDataFilter);
		}
		break;

	case EContentBrowserItemFlags::Type_File:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && !AssetDataFilter->bFilterExcludesAllAssets)
		{
			if (TSharedPtr<const FContentBrowserAliasItemDataPayload> AliasPayload = StaticCastSharedPtr<const FContentBrowserAliasItemDataPayload>(InItem.GetPayload()))
			{
				FAliasData* FoundAlias = AllAliases.Find(AliasPayload->Alias);
				// This should always be true except in the case where the item is deleted in the same tick that DoesItemPassFilter is called.
				// This is because the alias data source processes the deletion immediately, but the content browser deletion is queued until
				// next tick, causing them to be briefly out of sync.
				// An alternative solution would be to add a way to synchronously flush content browser updates, but that doesn't exist atm.
				if (FoundAlias)
				{
					return DoesAliasPassFilter(*FoundAlias, *AssetDataFilter);
				}
			}
		}
		break;
	}

	return false;
}

bool UContentBrowserAliasDataSource::DoesAliasPassFilter(const FAliasData& AliasData, const FContentBrowserCompiledAssetDataFilter& Filter) const
{
	// Create a fake asset data using the alias path instead of the asset's original path
	// AssetRegistry->IsAssetIncludedByFilter is effectively a static function and does not actually use AssetRegistry data
	FAssetData AliasAssetData(AliasData.PackageName, AliasData.PackagePath, AliasData.AssetData.AssetName, AliasData.AssetData.AssetClassPath, AliasData.AssetData.TagsAndValues.CopyMap());

	return (Filter.InclusiveFilter.IsEmpty() || AssetRegistry->IsAssetIncludedByFilter(AliasAssetData, Filter.InclusiveFilter)) // Passes Inclusive
		&& (Filter.ExclusiveFilter.IsEmpty() || !AssetRegistry->IsAssetIncludedByFilter(AliasAssetData, Filter.ExclusiveFilter)); // Passes Exclusive
}

TArray<FContentBrowserItemPath> UContentBrowserAliasDataSource::GetAliasesForPath(const FSoftObjectPath& ObjectPath) const
{
	TArray<FContentBrowserItemPath> OutAliases;

	if (const TArray<FName>* FoundAliases = AliasesForObjectPath.Find(ObjectPath))
	{
		OutAliases.Reserve(FoundAliases->Num());

		for (const FName Alias : *FoundAliases)
		{
			if (const FAliasData* AliasData = AllAliases.Find(FContentBrowserUniqueAlias(ObjectPath, Alias)))
			{
				OutAliases.Add(FContentBrowserItemPath(AliasData->InternalPath, EContentBrowserPathType::Internal));
			}
		}
	}

	return OutAliases;
}

bool UContentBrowserAliasDataSource::GetItemAliasedPath(const FContentBrowserItemData& InItemData, FSoftObjectPath& OutObjectPath) const
{
	if (InItemData.GetOwnerDataSource() == this && InItemData.IsFile())
	{
		if (const TSharedPtr<const FContentBrowserAliasItemDataPayload> AliasPayload = StaticCastSharedPtr<const FContentBrowserAliasItemDataPayload>(InItemData.GetPayload()))
		{
			if (AliasPayload->Alias.Key.IsValid())
			{
				OutObjectPath = AliasPayload->Alias.Key;
				return true;
			}
		}
	}
	return Super::GetItemAliasedPath(InItemData, OutObjectPath);
}

bool UContentBrowserAliasDataSource::HasAliasesForPath(const FSoftObjectPath& ObjectPath) const
{
	return AliasesForObjectPath.Contains(ObjectPath);
}

void UContentBrowserAliasDataSource::AddAliases(const FAssetData& Asset, const TArray<FName>& Aliases, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	AddAliasesImpl(Asset, Aliases, bInIsFromMetaData, bSkipPrimaryAssetValidation);
}

void UContentBrowserAliasDataSource::AddAliases(const FAssetData& Asset, const TArray<FContentBrowserLocalizedAlias>& Aliases, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	AddAliasesImpl(Asset, Aliases, bInIsFromMetaData, bSkipPrimaryAssetValidation);
}

template <typename AliasType>
void UContentBrowserAliasDataSource::AddAliasesImpl(const FAssetData& Asset, const TArray<AliasType>& Aliases, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	for (const AliasType& Alias : Aliases)
	{
		AddAlias(Asset, Alias, bInIsFromMetaData, bSkipPrimaryAssetValidation);
	}
}

void UContentBrowserAliasDataSource::AddAlias(const FAssetData& Asset, const FName Alias, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	AddAliasImpl(Asset, Alias, bInIsFromMetaData, bSkipPrimaryAssetValidation);
}

void UContentBrowserAliasDataSource::AddAlias(const FAssetData& Asset, const FContentBrowserLocalizedAlias& Alias, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	AddAliasImpl(Asset, Alias, bInIsFromMetaData, bSkipPrimaryAssetValidation);
}

template <typename AliasType>
void UContentBrowserAliasDataSource::AddAliasImpl(const FAssetData& Asset, const AliasType& AliasToAdd, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation)
{
	const FName Alias = ContentBrowserAliasDataSource::GetAliasName(AliasToAdd);
	const FText AliasDisplayNameOverride = ContentBrowserAliasDataSource::GetAliasDisplayNameOverride(AliasToAdd);

	auto LogErrorMessage = [&Alias, &Asset](const TCHAR* Reason)
	{
		UE_LOGF(LogContentBrowserAliasDataSource, Warning, "Cannot add alias %ls for %ls because: %ls", *Alias.ToString(), *Asset.GetObjectPathString(), Reason);
	};
	
	FContentBrowserUniqueAlias UniqueAlias = MakeTuple(Asset.GetSoftObjectPath(), Alias);
	if (AllAliases.Contains(UniqueAlias))
	{
		LogErrorMessage(TEXT("An alias with that name already exists"));
		return;
	}

	if (!bSkipPrimaryAssetValidation && !ContentBrowserAssetData::IsPrimaryAsset(Asset))
	{
		LogErrorMessage(TEXT("It is not a primary asset"));
		return;
	}

	const FString AliasString = Alias.ToString();
	if (AliasString[0] != TEXT('/'))
	{
		LogErrorMessage(TEXT("The alias is not a valid path"));
		return;
	}

	TArray<FString> Tokens;
	// TODO: figure out how to preserve spaces but also do some kind of invalid character checking that works even with fake object paths
	//Alias = ObjectTools::SanitizeObjectPath(Alias);
	AliasString.ParseIntoArray(Tokens, TEXT("/"), false);
	// Minimum valid tokens = [Empty, PackageName, ObjectName]
	if (Tokens.Num() < 3)
	{
		LogErrorMessage(TEXT("The alias is not a valid path"));
		return;
	}

	// Check for invalid empty tokens. Skip token 0 since it will be empty due to / at start
	for (int32 i = 1; i < Tokens.Num(); ++i)
	{
		if (Tokens[i].Len() == 0)
		{
			LogErrorMessage(TEXT("The alias is not a valid path"));
			return;
		}
	}
	FStringView AliasStringView(AliasString);
	// Tokens[1] is the root path without the slash prefix
	RootPathAdded(AliasStringView.Left(Tokens[1].Len() + 1));

	// PackagePath is everything before the last slash
	const FName PackagePath(AliasString.Len() - Tokens.Last().Len() - 1, AliasStringView.GetData());
	PathTree.CachePath(PackagePath, [this](FName AddedPath)
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateAssetFolderItem(AddedPath)));
	});

	AliasesInPackagePath.FindOrAdd(PackagePath).Add(UniqueAlias);
	AliasesForObjectPath.FindOrAdd(Asset.GetSoftObjectPath()).Add(Alias);
	FAliasData& AliasData = AllAliases.Add(UniqueAlias, FAliasData(Asset, PackagePath, AliasDisplayNameOverride.IsEmpty() ? FText::AsCultureInvariant(*Tokens.Last()) : AliasDisplayNameOverride, bInIsFromMetaData));
	AliasForInternalPath.Add(AliasData.InternalPath, UniqueAlias);
	QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateAssetFileItem(UniqueAlias)));

	// This logging might get out of control if there ends up being hundreds of thousands of aliases.
	//UE_LOGF(LogContentBrowserAliasDataSource, VeryVerbose, "Adding alias %ls for %ls", *AliasString, *Asset.ObjectPath.ToString());
}

void UContentBrowserAliasDataSource::RemoveFoldersRecursively(FStringView LeafFolder)
{
	// Make sure there's no assets here
	const FName LeafFolderName(LeafFolder.Len(), LeafFolder.GetData());
	if (LeafFolderName != NAME_None && !AliasesInPackagePath.Contains(LeafFolderName))
	{
		// Make sure there's no other child folders
		bool bHasChildren = false;
		PathTree.EnumerateSubPaths(LeafFolderName, [&bHasChildren](FName SubPath)
		{
			bHasChildren = true;
			return false;
		}, false);
		if (!bHasChildren)
		{
			// Remove folder from PathTree
			PathTree.RemovePath(LeafFolderName, [](FName Path) {});
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateAssetFolderItem(LeafFolderName)));

			// Check parent folder
			int32 LastSlash = INDEX_NONE;
			LeafFolder.FindLastChar(TEXT('/'), LastSlash);
			FStringView NewLeafFolder = LeafFolder.Left(LastSlash);

			// If the last slash is at the start of the string, this is the root folder
			if (LastSlash == 1)
			{
				RootPathRemoved(NewLeafFolder);
			}
			else
			{
				RemoveFoldersRecursively(NewLeafFolder);
			}
		}
	}
}

void UContentBrowserAliasDataSource::RemoveAlias(FName ObjectPath, const FName Alias)
{
	RemoveAlias(FSoftObjectPath(ObjectPath.ToString()), Alias);
}

void UContentBrowserAliasDataSource::RemoveAlias(const FSoftObjectPath& ObjectPath, const FName Alias)
{
    FContentBrowserUniqueAlias UniqueAlias = MakeTuple(ObjectPath, Alias);
	FAliasData AliasData;
	// Store a copy of the item data before it's removed for the MakeItemRemovedUpdate notification
	FContentBrowserItemData ItemData = CreateAssetFileItem(UniqueAlias);
	if (AllAliases.RemoveAndCopyValue(UniqueAlias, AliasData))
	{
		check(AliasData.AssetData.GetSoftObjectPath() == ObjectPath);
		{
			TArray<FName>& Aliases = AliasesForObjectPath[ObjectPath];
			Aliases.Remove(Alias);
			if (Aliases.IsEmpty())
			{
				AliasesForObjectPath.Remove(ObjectPath);
			}
		}

		{
			TArray<FContentBrowserUniqueAlias>& Aliases = AliasesInPackagePath[AliasData.PackagePath];
			Aliases.Remove(UniqueAlias);
			if (Aliases.IsEmpty())
			{
				AliasesInPackagePath.Remove(AliasData.PackagePath);
			}
		}

		AliasForInternalPath.Remove(AliasData.InternalPath);

		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(ItemData));

		FNameBuilder AliasAsString(Alias);
		FStringView AliasStringView(AliasAsString);

		int32 LastSlash = INDEX_NONE;
		AliasStringView.FindLastChar(TEXT('/'), LastSlash);
		RemoveFoldersRecursively(AliasStringView.Left(LastSlash));
	}
}

void UContentBrowserAliasDataSource::AddAliasFolderDisplayName(const FName AliasFolder, const FText& DisplayName)
{
	if (const FText* ExistingDisplayName = AliasFolderDisplayNames.Find(AliasFolder);
		ExistingDisplayName && ExistingDisplayName->IdenticalTo(DisplayName, ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		return;
	}

	AliasFolderDisplayNames.Add(AliasFolder, DisplayName);

	if (PathTree.PathExists(AliasFolder))
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFolderItem(AliasFolder)));
	}
}

void UContentBrowserAliasDataSource::RemoveAliasFolderDisplayName(const FName AliasFolder)
{
	AliasFolderDisplayNames.Remove(AliasFolder);

	if (PathTree.PathExists(AliasFolder))
	{
		QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFolderItem(AliasFolder)));
	}
}

void UContentBrowserAliasDataSource::RebuildAliases()
{
	PathTree = FPathTree();
	AllAliases.Reset();
	AliasesForObjectPath.Reset();
	AliasesInPackagePath.Reset();
	AliasFolderDisplayNames.Reset();
	FilterCache.Reset();
	RootPathVirtualTree.Reset();

	OnRebuildAliases().Broadcast();
}

void UContentBrowserAliasDataSource::RemoveAliases(FName ObjectPath)
{
	RemoveAliases(FSoftObjectPath(ObjectPath.ToString()));
}

void UContentBrowserAliasDataSource::RemoveAliases(const FSoftObjectPath& ObjectPath)
{
	const TArray<FName>* AliasesPtr = AliasesForObjectPath.Find(ObjectPath);
	if (AliasesPtr)
	{
		// Create a copy to not modify array during iteration
		const TArray<FName> Aliases = *AliasesPtr;
		for (const FName Alias : Aliases)
		{
			RemoveAlias(ObjectPath, Alias);
		}
	}
}

void UContentBrowserAliasDataSource::OnAssetAdded(const FAssetData& InAssetData)
{
	FString AliasTagValue;
	if (InAssetData.GetTagValue(AliasTagName, AliasTagValue))
	{
		TArray<FString> AliasList;
		AliasTagValue.ParseIntoArray(AliasList, TEXT(","));
		for (FString Alias : AliasList)
		{
			AddAlias(InAssetData, *Alias, true);
		}
	}
}

void UContentBrowserAliasDataSource::OnAssetRemoved(const FAssetData& InAssetData)
{
	RemoveAliases(InAssetData);
}

void UContentBrowserAliasDataSource::ReconcileAliasesFromMetaData(const FAssetData& Asset)
{
	const TArray<FName>* ExistingAliasesPtr = AliasesForObjectPath.Find(Asset.GetSoftObjectPath());
	if (ExistingAliasesPtr)
	{
		FContentBrowserUniqueAlias UniqueAlias = MakeTuple(Asset.GetSoftObjectPath(), NAME_None);
		
		FString AliasTagValue;
		if (Asset.GetTagValue(AliasTagName, AliasTagValue))
		{
			// Reconcile existing aliases from metadata vs new aliases from metadata
			TArray<FString> AliasesFromTag;
			AliasTagValue.ParseIntoArray(AliasesFromTag, TEXT(","));

			TArray<FName> AliasNamesFromTag;
			AliasNamesFromTag.Reserve(AliasesFromTag.Num());
			for (const FString& AliasFromTag : AliasesFromTag)
			{
				AliasNamesFromTag.Add(*AliasFromTag);
			}

			TArray<FName> AliasesOnlyInExisting, AliasesOnlyInNew;
			for (const FName Alias : *ExistingAliasesPtr)
			{
				UniqueAlias.Value = Alias;
				if (AllAliases[UniqueAlias].bIsFromMetaData)
				{
					if (!AliasNamesFromTag.Contains(Alias))
					{
						AliasesOnlyInExisting.Add(Alias);
					}
				}
			}

			for (const FName Alias : AliasNamesFromTag)
			{
				if (!ExistingAliasesPtr->Contains(Alias))
				{
					AliasesOnlyInNew.Add(Alias);
				}
			}

			for (const FName ExistingAlias : AliasesOnlyInExisting)
			{
				RemoveAlias(Asset.GetSoftObjectPath(), ExistingAlias);
			}
			AddAliases(Asset, AliasesOnlyInNew, true);
		}
		else
		{
			// If the tag was removed, then remove any existing aliases generated from metadata
			TArray<FName> ExistingAliases = *ExistingAliasesPtr;
			for (const FName Alias : ExistingAliases)
			{
				UniqueAlias.Value = Alias;
				if (AllAliases[UniqueAlias].bIsFromMetaData)
				{
					RemoveAlias(Asset.GetSoftObjectPath(), Alias);
				}
			}
		}
	}
	else
	{
		// If no existing aliases found, then check if new metadata was added
		OnAssetAdded(Asset);
	}
}

void UContentBrowserAliasDataSource::ReconcileAliasesForAsset(const FAssetData& Asset, const TArray<FName>& NewAliases)
{
	ReconcileAliasesForAssetImpl(Asset, NewAliases);
}

void UContentBrowserAliasDataSource::ReconcileAliasesForAsset(const FAssetData& Asset, const TArray<FContentBrowserLocalizedAlias>& NewAliases)
{
	ReconcileAliasesForAssetImpl(Asset, NewAliases);
}

template <typename AliasType>
void UContentBrowserAliasDataSource::ReconcileAliasesForAssetImpl(const FAssetData& Asset, const TArray<AliasType>& NewAliases)
{
	if (ensure(Asset.IsValid()))
	{
		const TArray<FName>* ExistingAliasesPtr = AliasesForObjectPath.Find(Asset.GetSoftObjectPath());
		if (ExistingAliasesPtr)
		{
			TArray<FName> AliasesOnlyInExisting;
			TArray<AliasType> AliasesOnlyInNew;
			FContentBrowserUniqueAlias UniqueAlias = MakeTuple(Asset.GetSoftObjectPath(), NAME_None);
			for (const FName Alias : *ExistingAliasesPtr)
			{
				UniqueAlias.Value = Alias;
				if (!AllAliases[UniqueAlias].bIsFromMetaData)
				{
					if (!NewAliases.ContainsByPredicate([&Alias](const AliasType& NewAlias) { return ContentBrowserAliasDataSource::GetAliasName(NewAlias) == Alias; }))
					{
						AliasesOnlyInExisting.Add(Alias);
					}
				}
			}

			for (const AliasType& Alias : NewAliases)
			{
				if (!ExistingAliasesPtr->Contains(ContentBrowserAliasDataSource::GetAliasName(Alias)))
				{
					AliasesOnlyInNew.Add(Alias);
				}
			}

			for (const FName ExistingAlias : AliasesOnlyInExisting)
			{
				RemoveAlias(Asset.GetSoftObjectPath(), ExistingAlias);
			}
			AddAliases(Asset, AliasesOnlyInNew);
		}
		else
		{
			AddAliases(Asset, NewAliases);
		}
	}
}

void UContentBrowserAliasDataSource::LogAliases() const
{
	TArray<FContentBrowserUniqueAlias> Aliases;
	AllAliases.GenerateKeyArray(Aliases);
	Aliases.Sort(
		[](const FContentBrowserUniqueAlias& A, const FContentBrowserUniqueAlias& B)
		{
			return A.Key == B.Key ? A.Value.LexicalLess(B.Value) : A.Key.LexicalLess(B.Key);
		});

	TStringBuilder<FName::StringBufferSize * 2> StringBuilder;
	for (const FContentBrowserUniqueAlias& Alias : Aliases)
	{
		StringBuilder.Reset();
		StringBuilder.Append(Alias.Key.ToString());
		StringBuilder.Append(TEXT(" -> \""));
		StringBuilder.Append(Alias.Value.ToString());
		StringBuilder.AppendChar(TCHAR('"'));
		UE_LOGF(LogContentBrowserAliasDataSource, Log, "%ls", StringBuilder.ToString());
	}
}

void UContentBrowserAliasDataSource::SetFilterShouldMatchCollectionContent(bool bInFilterShouldMatchCollectionContent)
{
	bFilterShouldMatchCollectionContent = bInFilterShouldMatchCollectionContent;
}

void UContentBrowserAliasDataSource::OnAssetUpdated(const FAssetData& InAssetData)
{
	ReconcileAliasesFromMetaData(InAssetData);
	UpdateAliasesCachedAssetData(InAssetData);
	MakeItemModifiedUpdate(InAssetData.GetSoftObjectPath());
}

void UContentBrowserAliasDataSource::UpdateAliasesCachedAssetData(const FAssetData& InAssetData)
{
	const FSoftObjectPath ObjectPath = InAssetData.GetSoftObjectPath();
	if (const TArray<FName>* Aliases = AliasesForObjectPath.Find(ObjectPath))
	{
		FContentBrowserUniqueAlias UniqueAlias = MakeTuple(ObjectPath, NAME_None);
		for (const FName Alias : *Aliases)
		{
			UniqueAlias.Value = Alias;
			if (FAliasData* AliasData = AllAliases.Find(UniqueAlias))
			{
				AliasData->AssetData = InAssetData;
			}
		}
	}
}

void UContentBrowserAliasDataSource::MakeItemModifiedUpdate(const FSoftObjectPath& ObjectPath)
{
	if (const TArray<FName>* Aliases = AliasesForObjectPath.Find(ObjectPath))
	{
		FContentBrowserUniqueAlias UniqueAlias = MakeTuple(ObjectPath, NAME_None);
		for (const FName Alias : *Aliases)
		{
			UniqueAlias.Value = Alias;
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateAssetFileItem(UniqueAlias)));
		}
	}
}

void UContentBrowserAliasDataSource::OnAssetLoaded(UObject* InAsset)
{
	if (InAsset && !InAsset->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing))
	{
		const FAssetData LoadedAssetData(InAsset);
		ReconcileAliasesFromMetaData(LoadedAssetData);
		UpdateAliasesCachedAssetData(LoadedAssetData);
		MakeItemModifiedUpdate(InAsset);
	}
}

void UContentBrowserAliasDataSource::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InObject && InObject->IsAsset() && AliasesForObjectPath.Contains(InObject))
	{
		const FAssetData LoadedAssetData(InObject);
		UpdateAliasesCachedAssetData(LoadedAssetData);
		MakeItemModifiedUpdate(InObject);
	}
}

bool UContentBrowserAliasDataSource::GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	return ContentBrowserAssetData::GetItemAttribute(ContentBrowserModule, this, InItem, InIncludeMetaData, InAttributeKey, OutAttributeValue);
}

bool UContentBrowserAliasDataSource::GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return ContentBrowserAssetData::GetItemAttributes(this, InItem, InIncludeMetaData, OutAttributeValues);
}

bool UContentBrowserAliasDataSource::GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
	{
		return ContentBrowserAssetData::GetItemPhysicalPath(this, InItem, OutDiskPath);
	}
	return false;
}

bool UContentBrowserAliasDataSource::IsItemDirty(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::IsItemDirty(this, InItem);
}

bool UContentBrowserAliasDataSource::CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserAliasItemDataPayload> AliasPayload = StaticCastSharedPtr<const FContentBrowserAliasItemDataPayload>(InItem.GetPayload()))
	{
		// Both the alias path and asset path must pass the writable folder filter and editable folder filter in order to be editable
		const TSharedRef<FPathPermissionList>& WritableFolderFilter = AssetTools->GetWritableFolderPermissionList();
		if (!WritableFolderFilter->PassesStartsWithFilter(AliasPayload->Alias.Value))
		{
			if (OutErrorMsg)
			{
				*OutErrorMsg = FText::Format(NSLOCTEXT("ContentBrowserAliasDataSource", "Error_FolderIsLocked", "Alias asset '{0}' is in a read only folder. Unable to edit read only assets."), FText::FromName(AliasPayload->Alias.Value));
			}
			return false;
		}

		if (UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem())
		{
			const TSharedRef<FPathPermissionList>& EditableFolderFilter = ContentBrowserDataSubsystem->GetEditableFolderPermissionList();
			if (!EditableFolderFilter->PassesStartsWithFilter(AliasPayload->Alias.Value))
			{
				if (OutErrorMsg)
				{
					*OutErrorMsg = FText::Format(NSLOCTEXT("ContentBrowserAliasDataSource", "Error_FolderIsNotEditable", "Alias asset '{0}' is in a folder that does not allow edits. Unable to edit read only assets."), FText::FromName(AliasPayload->Alias.Value));
				}
				return false;
			}
		}
	}
	return ContentBrowserAssetData::CanEditItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAliasDataSource::EditItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::EditItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAliasDataSource::BulkEditItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::EditItems(AssetTools, this, InItems);
}

bool UContentBrowserAliasDataSource::CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanViewItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAliasDataSource::ViewItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::ViewItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAliasDataSource::BulkViewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::ViewItems(AssetTools, this, InItems);
}

bool UContentBrowserAliasDataSource::CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanPreviewItem(AssetTools, this, InItem, OutErrorMsg);
}

bool UContentBrowserAliasDataSource::PreviewItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::PreviewItems(AssetTools, this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserAliasDataSource::BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserAssetData::PreviewItems(AssetTools, this, InItems);
}

bool UContentBrowserAliasDataSource::CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanSaveItem(AssetTools, this, InItem, InSaveFlags, OutErrorMsg);
}

bool UContentBrowserAliasDataSource::SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags)
{
	return ContentBrowserAssetData::SaveItems(AssetTools, this, MakeArrayView(&InItem, 1), InSaveFlags);
}

bool UContentBrowserAliasDataSource::BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags)
{
	return ContentBrowserAssetData::SaveItems(AssetTools, this, InItems, InSaveFlags);
}

bool UContentBrowserAliasDataSource::CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserAssetData::CanPrivatizeItem(AssetTools, AssetRegistry, this, InItem, OutErrorMsg);
}

bool UContentBrowserAliasDataSource::PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	return ContentBrowserAssetData::PrivatizeItems(AssetTools, AssetRegistry, this, MakeArrayView(&InItem, 1), InAssetAccessSpecifier);
}

bool UContentBrowserAliasDataSource::BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	return ContentBrowserAssetData::PrivatizeItems(AssetTools, AssetRegistry, this, InItems, InAssetAccessSpecifier);
}

bool UContentBrowserAliasDataSource::AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
	{
		return ContentBrowserAssetData::AppendItemReference(AssetRegistry, this, InItem, InOutStr);
	}
	return false;
}

bool UContentBrowserAliasDataSource::AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
	{
		return ContentBrowserAssetData::AppendItemObjectPath(AssetRegistry, this, InItem, InOutStr);
	}
	return false;
}

bool UContentBrowserAliasDataSource::AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
	{
		return ContentBrowserAssetData::AppendItemPackageName(AssetRegistry, this, InItem, InOutStr);
	}
	return false;
}

bool UContentBrowserAliasDataSource::UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	return ContentBrowserAssetData::UpdateItemThumbnail(this, InItem, InThumbnail);
}

bool UContentBrowserAliasDataSource::TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = ContentBrowserAssetData::GetAssetFileItemPayload(this, InItem))
	{
		OutCollectionId = AssetPayload->GetAssetData().GetSoftObjectPath();
		return true;
	}
	return false;
}

bool UContentBrowserAliasDataSource::GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier)
{
	return ContentBrowserAssetData::GetAssetAccessSpecifier(this, InItem, OutAssetAccessSpecifier);
}

bool UContentBrowserAliasDataSource::CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem)
{
	return ContentBrowserAssetData::CanModifyAssetAccessSpecifier(AssetTools, this, InItem);
}

bool UContentBrowserAliasDataSource::Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath)
{
	return false;
}

bool UContentBrowserAliasDataSource::Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData)
{
	if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = ContentBrowserAssetData::GetAssetFileItemPayload(this, InItem))
	{
		OutAssetData = AssetPayload->GetAssetData();
		return true;
	}
	return false;
}

bool UContentBrowserAliasDataSource::Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath)
{
	return false;
}

bool UContentBrowserAliasDataSource::Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath)
{
	return false;
}

void UContentBrowserAliasDataSource::RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter)
{
	FilterCache.RemoveUnusedCachedData(IDOwner, InVirtualPathsInUse, DataFilter);
}

void UContentBrowserAliasDataSource::ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	FilterCache.ClearCachedData(IDOwner);
}

FContentBrowserItemData UContentBrowserAliasDataSource::CreateAssetFolderItem(const FName InInternalFolderPath)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InInternalFolderPath, VirtualizedPath);

	const FString FolderItemName = FPackageName::GetShortName(InInternalFolderPath);
	return FContentBrowserItemData(
		this,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Asset,
		VirtualizedPath,
		*FolderItemName,
		AliasFolderDisplayNames.FindRef(InInternalFolderPath),
		MakeShared<FContentBrowserAssetFolderItemDataPayload>(InInternalFolderPath),
		{ InInternalFolderPath });
}

FContentBrowserItemData UContentBrowserAliasDataSource::CreateAssetFileItem(const FContentBrowserUniqueAlias& Alias)
{
	const FAliasData* AliasData = AllAliases.Find(Alias);
	// See UContentBrowserAliasDataSource::DoesItemPassFilter for more information on how this could fail
	if (AliasData)
	{
		LLM_SCOPE(ELLMTag::UI);
		FName VirtualizedPath;
		TryConvertInternalPathToVirtual(AliasData->InternalPath, VirtualizedPath);

		// Since AliasID is PackagePath/AssetName, AssetName should also be passed as the ItemName here. This provides the functionality of
		// being able to have multiple aliases with the same display name, while still showing their original asset name in the tooltip.
		return FContentBrowserItemData(
			this,
			EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Asset,
			VirtualizedPath,
			AliasData->AssetData.AssetName,
			AliasData->AliasDisplayName,
			MakeShared<FContentBrowserAliasItemDataPayload>(AliasData->AssetData, Alias),
			{ AliasData->InternalPath });
	}
	return FContentBrowserItemData();
}
