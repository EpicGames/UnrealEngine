// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserAliasDataSourceModule.h"

void FContentBrowserAliasDataSourceModule::StartupModule()
{
	UObject::GetMetaDataTagsForAssetRegistry().Add(UContentBrowserAliasDataSource::AliasTagName);

	AliasDataSource.Reset(NewObject<UContentBrowserAliasDataSource>(GetTransientPackage(), "AliasData"));
	AliasDataSource->Initialize();
}

void FContentBrowserAliasDataSourceModule::PreUnloadCallback()
{
	AliasDataSource.Reset();
}

UContentBrowserAliasDataSource* FContentBrowserAliasDataSourceModule::TryGetAliasDataSource()
{
	// GExitPurge guard required because crash inspecting object after GExitPurge
	if (!GExitPurge)
	{
		if (UContentBrowserAliasDataSource* Obj = AliasDataSource.Get())
		{
			if (IsValid(Obj))
			{
				return Obj;
			}
		}
	}

	return nullptr;
}

IMPLEMENT_MODULE(FContentBrowserAliasDataSourceModule, ContentBrowserAliasDataSource);
