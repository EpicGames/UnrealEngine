// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserAssetDataSource.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserLocalizedAlias.h"

#include "AssetRegistry/PathTree.h"
#include "UObject/Package.h"
#include "ContentBrowserAliasDataSource.generated.h"

#define UE_API CONTENTBROWSERALIASDATASOURCE_API

class IAssetRegistry;
struct FPropertyChangedEvent;

DECLARE_LOG_CATEGORY_EXTERN(LogContentBrowserAliasDataSource, Log, All);

class IAssetTools;
struct FContentBrowserCompiledAssetDataFilter;
class FContentBrowserModule;

/** A unique alias is a pair of SourceObjectPath:AliasPath, eg /Game/MyAsset.MyAsset:/Game/SomeFolder/MyAlias */
typedef TPair<FSoftObjectPath, FName> FContentBrowserUniqueAlias;

class FContentBrowserAliasItemDataPayload : public FContentBrowserAssetFileItemDataPayload
{
public:
	FContentBrowserAliasItemDataPayload(const FAssetData& InAssetData, const FContentBrowserUniqueAlias& InAlias)
		: FContentBrowserAssetFileItemDataPayload(InAssetData), Alias(InAlias)
	{
	}

	FContentBrowserUniqueAlias Alias;
};

/**
 * A companion to the ContentBrowserAssetDataSource which can display assets in folders other than their actual folder. Aliases mimic their source asset as closely as possible,
 * including editing, saving, thumbnails, and more. Some behavior is restricted such as moving or deleting an alias item.
 * 
 * Aliases can either be created automatically by tagging the asset with the value defined by AliasTagName and giving it a comma-separated list of aliases,
 * or manually managed by calling AddAlias and RemoveAlias. ReconcileAliasesForAsset is provided as a helper function to automatically update new/removed aliases for an existing asset.
 *
 */
UCLASS(MinimalAPI)
class UContentBrowserAliasDataSource : public UContentBrowserDataSource
{
	GENERATED_BODY()

public:
	/** The metadata tag to set for the AliasDataSource to automatically create aliases for an asset */
	static UE_API FName AliasTagName;

	// ~ Begin UContentBrowserDataSource interface
	UE_API void Initialize(const bool InAutoRegister = true);
	UE_API virtual void Shutdown() override;

	UE_API virtual void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) override;
	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;
	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;
	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags, const FContentBrowserFolderContentsFilter& ContentsFilter) override;

	UE_API virtual bool DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter) override;

	using UContentBrowserDataSource::GetAliasesForPath;
	UE_API virtual TArray<FContentBrowserItemPath> GetAliasesForPath(const FSoftObjectPath& InInternalPath) const override;
	UE_API virtual bool GetItemAliasedPath(const FContentBrowserItemData& InItemData, FSoftObjectPath& OutObjectPath) const override;

	UE_API bool HasAliasesForPath(const FSoftObjectPath& InInternalPath) const;

	UE_API virtual bool GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue) override;
	UE_API virtual bool GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues) override;
	UE_API virtual bool GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath) override;
	UE_API virtual bool IsItemDirty(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;
	UE_API virtual bool EditItem(const FContentBrowserItemData& InItem) override;
	UE_API virtual bool BulkEditItems(TArrayView<const FContentBrowserItemData> InItems) override;
	
	UE_API virtual bool CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;
	UE_API virtual bool ViewItem(const FContentBrowserItemData& InItem) override;
	UE_API virtual bool BulkViewItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;
	UE_API virtual bool PreviewItem(const FContentBrowserItemData& InItem) override;
	UE_API virtual bool BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg) override;
	UE_API virtual bool SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags) override;
	UE_API virtual bool BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags) override;

	UE_API virtual bool CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;
	UE_API virtual bool PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private) override;
	UE_API virtual bool BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private) override;

	UE_API virtual bool AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr) override;
	UE_API virtual bool AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr) override;
	UE_API virtual bool AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr) override;
	UE_API virtual bool UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail) override;

	UE_API virtual bool TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId) override;

	virtual bool GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier) override;
	virtual bool CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem) override;

	// Legacy functions seem necessary for FrontendFilters to work
	UE_API virtual bool Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath) override;
	UE_API virtual bool Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData) override;
	UE_API virtual bool Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath) override;
	UE_API virtual bool Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath) override;
	// ~ End UContentBrowserDataSource interface

	UE_API virtual void RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) override;

	UE_API virtual void ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner) override;

	/** 
	 * Add a list of aliases for a given asset.
	 *
	 * @param Asset 						The asset to add aliases for.
	 * @param Aliases 						The aliases to add
	 * @param bInIsFromMetaData 			Should only be `true` if the list of aliases came from the `AliasTagName` metadata.
	 * @param bSkipPrimaryAssetValidation 	Use this if the asset being added is not a primary asset/is a re-director but should still be included in the Content Browser. (e.g. Verse classes)
	 */
	UE_API void AddAliases(const FAssetData& Asset, const TArray<FName>& Aliases, const bool bInIsFromMetaData = false, const bool bSkipPrimaryAssetValidation = false);
	UE_API void AddAliases(const FAssetData& Asset, const TArray<FContentBrowserLocalizedAlias>& Aliases, const bool bInIsFromMetaData = false, const bool bSkipPrimaryAssetValidation = false);
	/** Add an alias for a given asset. bInIsFromMetaData should only be true if the alias came from the AliasTagName metadata. */
	UE_API void AddAlias(const FAssetData& Asset, const FName Alias, const bool bInIsFromMetaData = false, const bool bSkipPrimaryAssetValidation = false);
	UE_API void AddAlias(const FAssetData& Asset, const FContentBrowserLocalizedAlias& Alias, const bool bInIsFromMetaData = false, const bool bSkipPrimaryAssetValidation = false);
	/** Remove the given alias from the data source */
	UE_API void RemoveAlias(const FSoftObjectPath& ObjectPath, const FName Alias);
	/** Remove all aliases for the given object */
	UE_API void RemoveAliases(const FSoftObjectPath& ObjectPath);
	/** Remove all aliases for the given asset */
	void RemoveAliases(const FAssetData& Asset) { RemoveAliases(Asset.GetSoftObjectPath()); }

	UE_DEPRECATED(5.1, "FNames containing full asset paths are deprecated, use FSoftObjectPath instead")
	UE_API void RemoveAlias(const FName ObjectPath, const FName Alias);
	UE_DEPRECATED(5.1, "FNames containing full asset paths are deprecated, use FSoftObjectPath instead")
	UE_API void RemoveAliases(const FName ObjectPath);

	/**
	 * Add a display name override for the given alias folder, eg /MyAliases.
	 * @note To provide a display name override for an alias itself, use the overloads that take a FContentBrowserLocalizedAlias.
	 */
	UE_API void AddAliasFolderDisplayName(const FName AliasFolder, const FText& DisplayName);
	/**
	 * Remove a display name override for the given alias folder, eg /MyAliases.
	 */
	UE_API void RemoveAliasFolderDisplayName(const FName AliasFolder);

	/** When called, removes all aliases and triggers delegate for various systems to re-add aliases */
	UE_API void RebuildAliases();

	/** Broadcast after RebuildAliases() called to allow systems to re-add aliases */
	FSimpleMulticastDelegate& OnRebuildAliases() { return RebuildAliasesDelegate; }

	/** Get all aliases from metadata for the given asset, then calls AddAlias or RemoveAlias for every alias that doesn't match the stored data. */
	UE_API void ReconcileAliasesFromMetaData(const FAssetData& Asset);
	/** Calls AddAlias or RemoveAlias for every alias that doesn't match the stored data for the given asset. */
	UE_API void ReconcileAliasesForAsset(const FAssetData& Asset, const TArray<FName>& NewAliases);
	UE_API void ReconcileAliasesForAsset(const FAssetData& Asset, const TArray<FContentBrowserLocalizedAlias>& NewAliases);

	/** Logs all the content browser aliases */
	UE_API void LogAliases() const;

	UE_API void SetFilterShouldMatchCollectionContent(bool bInFilterShouldMatchCollectionContent);

protected:
	UE_API virtual void BuildRootPathVirtualTree() override;

private:
	template <typename AliasType>
	void AddAliasImpl(const FAssetData& Asset, const AliasType& Alias, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation);
	template <typename AliasType>
	void AddAliasesImpl(const FAssetData& Asset, const TArray<AliasType>& Aliases, const bool bInIsFromMetaData, const bool bSkipPrimaryAssetValidation);
	template <typename AliasType>
	void ReconcileAliasesForAssetImpl(const FAssetData& Asset, const TArray<AliasType>& NewAliases);

	UE_API void OnAssetAdded(const FAssetData& InAssetData);
	UE_API void OnAssetRemoved(const FAssetData& InAssetData);
	UE_API void OnAssetUpdated(const FAssetData& InAssetData);
	UE_API void OnAssetLoaded(UObject* InAsset);
	UE_API void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	/** Helper function to remove a folder from the PathTree including all parent folders that are now empty as a result of the removal */
	UE_API void RemoveFoldersRecursively(FStringView LeafFolder);

	UE_API void UpdateAliasesCachedAssetData(const FAssetData& InAssetData);
	UE_API void MakeItemModifiedUpdate(const FSoftObjectPath& ObjectPath);

	UE_API FContentBrowserItemData CreateAssetFolderItem(const FName InFolderPath);
	UE_API FContentBrowserItemData CreateAssetFileItem(const FContentBrowserUniqueAlias& Alias);

	IAssetRegistry* AssetRegistry = nullptr;
	IAssetTools* AssetTools = nullptr;
	FContentBrowserModule* ContentBrowserModule = nullptr;

	struct FAliasData
	{
		FAliasData() {}
		FAliasData(const FAssetData& InAssetData, const FName InPackagePath, const FText& InDisplayName, const bool bInIsFromMetaData = false)
			: AssetData(InAssetData), PackagePath(InPackagePath), AliasDisplayName(InDisplayName), bIsFromMetaData(bInIsFromMetaData)
		{
			FNameBuilder AssetNameBuilder(InAssetData.AssetName);
			{
				// Add a hash of the real asset path to the asset name to ensure uniqueness of the FContentBrowserItemKey
				// so that two different assets with the same name and the same alias both show up in the content browser
				FNameBuilder AssetPathBuilder;
				InAssetData.AppendObjectPath(AssetPathBuilder);
				AssetNameBuilder.Appendf(TEXT("_%08X"), GetTypeHash(AssetPathBuilder.ToView()));
			}
			{
				FNameBuilder PathBuilder(PackagePath);
				PathBuilder << TEXT('/');
				PathBuilder << AssetNameBuilder.ToView();
				PackageName = FName(PathBuilder.ToView());

				PathBuilder << TEXT('.');
				PathBuilder << AssetNameBuilder.ToView();
				InternalPath = FName(PathBuilder.ToView());
			}
		}

		/** The source asset for this alias */
		FAssetData AssetData;
		/** The folder path that contains the alias, /MyAliases */
		FName PackagePath;
		/** PackagePath/SourceAssetName, /MyAliases/SomeAsset_Hash  */
		FName PackageName;
		/** PackageName.SourceAssetName, /MyAliases/SomeAsset_Hash.SomeAsset_Hash */
		FName InternalPath;
		/** A non-unique display name for this alias */
		FText AliasDisplayName;
		/** Whether this alias was generated from package metadata or manually through the C++ interface */
		bool bIsFromMetaData = false;
	};
	UE_API bool DoesAliasPassFilter(const FAliasData& AliasData, const FContentBrowserCompiledAssetDataFilter& Filter) const;

	/** The full folder hierarchy for all alias paths */
	FPathTree PathTree;
	/** Alias data keyed by their full alias path, eg /Game/MyData/Aliases/SourceMesh */
	TMap<FContentBrowserUniqueAlias, FAliasData> AllAliases;
	/** A list of alias paths to display for each asset, eg /Game/Meshes/SourceMesh.SourceMesh */
	TMap<FSoftObjectPath, TArray<FName>> AliasesForObjectPath;
	/** A list of alias paths to display for each folder, eg /Game/MyData/Aliases */
	TMap<FName, TArray<FContentBrowserUniqueAlias>> AliasesInPackagePath;
	/** The alias path to display for each file */
	TMap<FName, FContentBrowserUniqueAlias> AliasForInternalPath;
	/** Alias folder display names keyed against their alias path, eg /Game/MyData/Aliases/ */
	TMap<FName, FText> AliasFolderDisplayNames;

	UContentBrowserAssetDataSource::FAssetDataSourceFilterCache FilterCache;

	/** Delegate broadcast after all aliases removed to give chance for systems to re-add aliases */
	FSimpleMulticastDelegate RebuildAliasesDelegate;

	/** If true, EnumerateItemsMatchingFilter will include aliases of items contained in collections included in the filter */
	bool bFilterShouldMatchCollectionContent = true;
};

#undef UE_API
