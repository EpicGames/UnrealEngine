// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserAliasDataSource.h"
#include "Modules/ModuleManager.h"
#include "UObject/StrongObjectPtr.h"

class FContentBrowserAliasDataSourceModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override;
	virtual void PreUnloadCallback() override;

	UE_DEPRECATED(5.4, "Call TryGetAliasDataSource instead.")
	TWeakObjectPtr<UContentBrowserAliasDataSource> GetAliasDataSource() { return AliasDataSource.Get(); }

	CONTENTBROWSERALIASDATASOURCE_API UContentBrowserAliasDataSource* TryGetAliasDataSource();

private:
	TStrongObjectPtr<UContentBrowserAliasDataSource> AliasDataSource;
};
