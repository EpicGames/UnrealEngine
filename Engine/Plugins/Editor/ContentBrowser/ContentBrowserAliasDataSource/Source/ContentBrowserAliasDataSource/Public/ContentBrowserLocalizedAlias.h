// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

/** Container for an alias and its optional display name override */
struct FContentBrowserLocalizedAlias
{
	FName Alias;
	FText DisplayName;

	bool operator==(const FContentBrowserLocalizedAlias& Other) const
	{
		return Alias == Other.Alias;
	}

	bool operator!=(const FContentBrowserLocalizedAlias& Other) const
	{
		return Alias != Other.Alias;
	}

	friend uint32 GetTypeHash(const FContentBrowserLocalizedAlias& This)
	{
		return GetTypeHash(This.Alias);
	}
};
